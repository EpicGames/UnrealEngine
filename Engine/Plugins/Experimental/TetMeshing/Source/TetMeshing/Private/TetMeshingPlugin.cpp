// Copyright Epic Games, Inc. All Rights Reserved.

#include "TetMeshingPlugin.h"

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"


DEFINE_LOG_CATEGORY(LogTetMeshing);


class FTetMeshingPlugin : public ITetMeshingPlugin
{
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};

IMPLEMENT_MODULE( FTetMeshingPlugin, TetMeshing )



void FTetMeshingPlugin::StartupModule()
{
	
}


void FTetMeshingPlugin::ShutdownModule()
{
	
}



