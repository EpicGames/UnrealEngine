// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Generate/TetMeshGenerator.h"
#include "VectorTypes.h"
#include "Math/IntVector.h"

// Implementation of "Isosurface Stuffing: Fast Tetrahedral Meshes with Good Dihedral Angles"
// By Francois Labelle and Jonathan Richard Shewchuk
// https://people.eecs.berkeley.edu/~jrs/papers/stuffing.pdf

namespace UE
{
namespace Geometry
{

enum class ERootFindingMethod : uint8
{
	Bisection
	// TODO: MultiLabelBisection?, Lerp?, LerpSteps?
};

template<typename RealType, ERootFindingMethod RootMethod = ERootFindingMethod::Bisection>
class TIsosurfaceStuffing : public TTetMeshGenerator<RealType>
{
public:
	using TVec3 = UE::Math::TVector<RealType>;
	using TTetMeshGenerator<RealType>::Vertices;

	// Parameters for generation
	UE::Math::TBox<RealType> Bounds;
	RealType CellSize;
	// Implicit is the implicit function defining the desired output shape, typically a fast winding or signed distance function
	// It should be larger than IsoValue inside the desired shape, and smaller than IsoValue outside
	TFunction<RealType(TVec3)> Implicit;
	RealType IsoValue = 0;
	// Number of steps to take when root finding
	int32 RootModeSteps = 5;
	

	virtual TIsosurfaceStuffing& Generate() override
	{
		TVec3 DimsFloat = Bounds.GetSize() / CellSize;
		FIntVector3 Dims; // number of cubes in lattice per dimension
		for (int32 DimIdx = 0; DimIdx < 3; ++DimIdx)
		{
			Dims[DimIdx] = FMath::CeilToInt32(DimsFloat[DimIdx]) + 1;
		}

		// TODO: Implement continuation-style generation, to sample grid selectively from seed points instead of iterating over the whole grid
		for (int32 X = 0; X < Dims.X; ++X)
		{
			for (int32 Y = 0; Y < Dims.Y; ++Y)
			{
				for (int32 Z = 0; Z < Dims.Z; ++Z)
				{
					FIntVector3 CenterA(X, Y, Z);
					FGridPt GridA(CenterA, true);
					FGridPtData CenterData[2];
					TVec3 CenterPos[2];
					FGridPtData CornerData[4];
					TVec3 CornerPos[4];
					CenterData[0] = GetGridPtData(GridA, CenterPos[0], true);
					// TODO: GetGridPtData w/ warping has necessarily checked all edges touching the cell center,
					// so should be able to tell us if we can skip the rest of the processing for this cell

					// Stuff the four BCC tetrahedra connecting center X,Y,Z to the adjacent center in each direction
					for (int32 DimIdx = 0; DimIdx < 3; ++DimIdx)
					{
						FIntVector3 CenterB = CenterA;
						CenterB[DimIdx] += 1;
						if (CenterB[DimIdx] >= Dims[DimIdx])
						{
							continue; // nothing to extend to in this direction
						}
						FGridPt GridB(CenterB, true);
						CenterData[1] = GetGridPtData(GridB, CenterPos[1], true);
						int32 XDim = (DimIdx + 1) % 3;
						int32 YDim = (DimIdx + 2) % 3;
						FGridPt CornerPts[4];
						for (int32 CornerIdx = 0; CornerIdx < 4; ++CornerIdx)
						{
							int32 DY = (CornerIdx & 2) >> 1; // second two corners are 'top' points
							int32 DX = DY ^ (CornerIdx & 1); // first and last corners are 'left' points

							FIntVector Corner = CenterB;
							Corner[XDim] += DX;
							Corner[YDim] += DY;
							CornerPts[CornerIdx] = FGridPt(Corner, false);
							CornerData[CornerIdx] = GetGridPtData(CornerPts[CornerIdx], CornerPos[CornerIdx], true);
						}
						FGridPtData Data[4];
						Data[0] = CenterData[0];
						Data[1] = CenterData[1];
						FGridPt Pts[4];
						Pts[0] = GridA;
						Pts[1] = GridB;
						for (int32 CornerIdx = 0, LastCorner = 3; CornerIdx < 4; LastCorner = CornerIdx++)
						{
							Data[2] = CornerData[LastCorner];
							Data[3] = CornerData[CornerIdx];
							Pts[2] = CornerPts[LastCorner];
							Pts[3] = CornerPts[CornerIdx];
							ProcessTet(Pts, Data);
						}
					}
				}
			}
		}

		return *this;
	}

	virtual void Reset() override
	{
		TTetMeshGenerator<RealType>::Reset();
		ComputedGridPts.Reset();
		ComputedCutPts.Reset();
	}

protected:

	struct FGridPt
	{
		FIntVector3 Idx;
		bool bIsCenter;

		FGridPt(FIntVector3 Idx, bool bIsCenter) : Idx(Idx), bIsCenter(bIsCenter)
		{}
		FGridPt() : Idx(-1, -1, -1), bIsCenter(false)
		{}
		FGridPt(int32 X, int32 Y, int32 Z, bool bIsCenter) : Idx(X, Y, Z), bIsCenter(bIsCenter)
		{}
	};

	struct FGridPtData
	{
		int32 VertexIndex = INDEX_NONE; // Output vertex index; only applicable if Label >= 0
		int16 Label; // Negative = Outside, 0 = On surfaces, >=1 = inside surface w/ label indicated
		bool bWarpProcessed = false;
	};

	struct FCutData
	{
		int32 VertexIndex = INDEX_NONE; // Output vertex index; can be unset
		TVec3 CutPos;
		// TODO: consider just storing Alpha instead of CutPos:
		// RealType Alpha; // Interpolation parameter in the canonical direction (Center->Corner or Low->High)
	};

	TMap<int64, FGridPtData> ComputedGridPts;
	TMap<int64, FCutData> ComputedCutPts;

	// Get grid coordinates s.t. Cell 0,0,0's center is at the minimum corner of Bounds
	inline TVec3 GetUnwarpedGridPos(const FGridPt& Pt) const
	{
		RealType HalfCell = CellSize * .5;
		return Bounds.Min + TVec3(Pt.Idx.X * CellSize, Pt.Idx.Y * CellSize, Pt.Idx.Z * CellSize) - float(!Pt.bIsCenter) * TVec3(HalfCell, HalfCell, HalfCell);
	}
	TVec3 ComputeEdgeCutPos(TVec3 APos, RealType AVal, TVec3 BPos, RealType BVal) const
	{
		if constexpr (RootMethod == ERootFindingMethod::Bisection)
		{
			checkSlow(FMath::Sign(AVal - IsoValue) != FMath::Sign(BVal - IsoValue)); // TODO: does not apply if multi-class?
			if (AVal > BVal)
			{
				Swap(AVal, BVal);
				Swap(APos, BPos);
			}

			TVec3 PIso;
			for (int Step = 0; Step < RootModeSteps; ++Step)
			{
				PIso = (APos + BPos) * 0.5;
				double MidF = Implicit(PIso);
				if (MidF < IsoValue)
				{
					APos = PIso; AVal = MidF;
				}
				else
				{
					BPos = PIso; BVal = MidF;
				}
			}
			PIso = (APos + BPos) * 0.5;
			return PIso;
		}
		else
		{
			check(false); // Unimplemented root-finding method
			return TVec3(0, 0, 0);
		}
	}

	/// Edge and Point ID Packing Scheme:
	/// We use a bit-packing scheme to generate unique int64 IDs for grid points and canonical BCC edges.
	/// This is adapted from the scheme used by MarchingCubes.h, extended to handle the BCC grid's diagonals and center points.
	/// 
	/// Grid points are defined by an X,Y,Z and a bool indicating if they are centered.
	/// We assume the grid dimensions cannot exceed 2^15 on any axis, so we can pack grid points
	/// in the first 49 bits of an int64: 16 bits per X, Y, Z, and 1 bit for the bIsCenter flag.
	/// Edge IDs are given by one of the grid points bitwise or'd with a 'Tag' indicating the attached edge.
	/// To make sure each edge only has one possible ID, by convention, the grid point for an edge is chosen by these rules:
	///  - If the edge is a 'long' BCC edge, along a major axis, use the grid point with the lowest coordinates.
	///  - Otherwise, for 'short' BCC edges, connecting centered to non-centered points, always use the centered point.
	/// Edge tags are numbered as follows:
	///  - 1,2,3 for the long edges along the X,Y,Z axes respectively, as returned by GetMajorAxisEdgeTag.
	///  - 4-11 for the short edges, as returned by GetCornerEdgeTag(Difference in coordinates).
	///  (Note that the coordinate difference from center->corner is always 0 or 1 in each dimension.)
	/// The tag number is bit-shifted by 50 to safely combine with the Point ID.
	/// 
	/// In sum, the bits of an Edge ID are laid out as:
	/// [Bits 53-50: 'Tag' indicating which edge] [Bits 48-0: Point ID for a grid point attached to the edge]
	/// And Point ID is laid out as:
	/// [Bit 48: bIsCenter flag] [Bits 47-32: Z coord] [Bits 31-16: Y coord] [Bits 15-0: X coord]
	
	// Axis should be 0, 1, or 2 for X, Y, or Z
	static constexpr int64 GetMajorAxisEdgeTag(int64 Axis)
	{
		return int64(Axis + 1) << 50;
	}
	// Delta should be 0 or 1 for each coordinate
	static constexpr int64 GetCornerEdgeTag(FIntVector3 Delta)
	{
		int32 CornerEncoded = Delta.X + Delta.Y * 2 + Delta.Z * 4 + 4;
		return int64(CornerEncoded) << 50;
	}

	// Return a unique ID for the BCC edge connecting A to B (note: assumes A,B are connected in the BCC lattice)
	static int64 GetEdgeID(FGridPt A, FGridPt B)
	{
		constexpr int64 EDGE_X = GetMajorAxisEdgeTag(0);
		constexpr int64 EDGE_Y = GetMajorAxisEdgeTag(1);
		constexpr int64 EDGE_Z = GetMajorAxisEdgeTag(2);
		if (A.bIsCenter == B.bIsCenter)
		{
			if (A.Idx.X != B.Idx.X)
			{
				return GetGridID(FMath::Min(A.Idx.X, B.Idx.X), A.Idx.Y, A.Idx.Z, A.bIsCenter) | EDGE_X;
			}
			else if (A.Idx.Y != B.Idx.Y)
			{
				return GetGridID(A.Idx.X, FMath::Min(A.Idx.Y, B.Idx.Y), A.Idx.Z, A.bIsCenter) | EDGE_Y;
			}
			else
			{
				return GetGridID(A.Idx.X, A.Idx.Y, FMath::Min(A.Idx.Z, B.Idx.Z), A.bIsCenter) | EDGE_Z;
			}
		}
		else
		{
			FIntVector3 Delta, CenterIdx;
			if (B.bIsCenter)
			{
				Delta = A.Idx - B.Idx;
				CenterIdx = B.Idx;
			}
			else
			{
				Delta = B.Idx - A.Idx;
				CenterIdx = A.Idx;
			}
			checkSlow(Delta.X >= 0 && Delta.X <= 1 && Delta.Y >= 0 && Delta.Y <= 1 && Delta.Z >= 0 && Delta.Z <= 1);
			int64 CornerTag = GetCornerEdgeTag(Delta);
			return GetGridID(CenterIdx, true) | CornerTag;
		}
	}

	static inline int64 GetGridID(const FGridPt& Pt)
	{
		return GetGridID(Pt.Idx, Pt.bIsCenter);
	}
	static inline int64 GetGridID(FIntVector3 Idx, bool bIsCenter)
	{
		return GetGridID(Idx.X, Idx.Y, Idx.Z, bIsCenter);
	}
	static int64 GetGridID(int32 X, int32 Y, int32 Z, bool bIsCenter)
	{
		return ((int64)X & 0xFFFF) | (((int64)Y & 0xFFFF) << 16) | (((int64)Z & 0xFFFF) << 32) | (((int64)bIsCenter << 48));
	}

	static void GetEdges(FGridPt Pt, int64 EdgeIDsOut[14], FGridPt OtherPtOut[14])
	{
		// first six == along major axes, last 8 == corners
		constexpr int64 EDGE_X = GetMajorAxisEdgeTag(0);
		constexpr int64 EDGE_Y = GetMajorAxisEdgeTag(1);
		constexpr int64 EDGE_Z = GetMajorAxisEdgeTag(2);
		int64 PtID = GetGridID(Pt);
		OtherPtOut[0] = FGridPt(Pt.Idx.X + 1, Pt.Idx.Y, Pt.Idx.Z, Pt.bIsCenter);
		EdgeIDsOut[0] = PtID | EDGE_X;
		OtherPtOut[1] = FGridPt(Pt.Idx.X, Pt.Idx.Y + 1, Pt.Idx.Z, Pt.bIsCenter);
		EdgeIDsOut[1] = PtID | EDGE_Y;
		OtherPtOut[2] = FGridPt(Pt.Idx.X, Pt.Idx.Y, Pt.Idx.Z + 1, Pt.bIsCenter);
		EdgeIDsOut[2] = PtID | EDGE_Z;
		OtherPtOut[3] = FGridPt(Pt.Idx.X - 1, Pt.Idx.Y, Pt.Idx.Z, Pt.bIsCenter);
		EdgeIDsOut[3] = GetGridID(OtherPtOut[3]) | EDGE_X;
		OtherPtOut[4] = FGridPt(Pt.Idx.X, Pt.Idx.Y - 1, Pt.Idx.Z, Pt.bIsCenter);
		EdgeIDsOut[4] = GetGridID(OtherPtOut[4]) | EDGE_Y;
		OtherPtOut[5] = FGridPt(Pt.Idx.X, Pt.Idx.Y, Pt.Idx.Z - 1, Pt.bIsCenter);
		EdgeIDsOut[5] = GetGridID(OtherPtOut[5]) | EDGE_Z;
		if (Pt.bIsCenter)
		{
			int32 OutPtIdx = 6;
			for (int32 XOff = 0; XOff < 2; ++XOff)
			{
				for (int32 YOff = 0; YOff < 2; ++YOff)
				{
					for (int32 ZOff = 0; ZOff < 2; ++ZOff)
					{
						OtherPtOut[OutPtIdx] = FGridPt(Pt.Idx.X + XOff, Pt.Idx.Y + YOff, Pt.Idx.Z + ZOff, false);
						EdgeIDsOut[OutPtIdx] = PtID | GetCornerEdgeTag(FIntVector3(XOff, YOff, ZOff));
						OutPtIdx++;
					}
				}
			}
			return;
		}
		else
		{
			// Connect to center, all combos of Idx -1,-0 per dim
			int32 OutPtIdx = 6;
			for (int32 XOff = 0; XOff < 2; ++XOff)
			{
				for (int32 YOff = 0; YOff < 2; ++YOff)
				{
					for (int32 ZOff = 0; ZOff < 2; ++ZOff)
					{
						OtherPtOut[OutPtIdx] = FGridPt(Pt.Idx.X - XOff, Pt.Idx.Y - YOff, Pt.Idx.Z - ZOff, true);
						EdgeIDsOut[OutPtIdx] = GetGridID(OtherPtOut[OutPtIdx]) | GetCornerEdgeTag(FIntVector3(XOff, YOff, ZOff));
						checkSlow(EdgeIDsOut[OutPtIdx] == GetEdgeID(Pt, OtherPtOut[OutPtIdx]));
						OutPtIdx++;
					}
				}
			}
			return;
		}
	}

	// TODO: Update this for the multi-label case?
	inline int16 ImplicitValueToLabel(RealType Value) const
	{
		return (int16)TMathUtil<RealType>::SignAsInt(Value - IsoValue);
	}

	// TODO: Note this is incompatible with Lerp-based iso-surface finding
	inline RealType LabelToImplicitValue(int16 Label) const
	{
		return RealType(Label) + IsoValue;
	}

	FGridPtData GetGridPtData(const FGridPt& Pt, TVec3& PosOut, bool bAllowWarp = true)
	{
		int64 GID = GetGridID(Pt);
		FGridPtData* GridPtData = ComputedGridPts.Find(GID);
		FGridPtData ToReturn;
		if (!GridPtData)
		{
			PosOut = GetUnwarpedGridPos(Pt);
			RealType Value = Implicit(PosOut);
			int16 Label = ImplicitValueToLabel(Value);

			int32 VertexIdx = -1;
			if (Label >= 0)
			{
				VertexIdx = Vertices.Add(PosOut);
			}
			ToReturn = FGridPtData{ VertexIdx, Label, false };
			if (!bAllowWarp) // if we allow warping, the point will instead be added after considering the warp
			{
				// TODO: possibly not worth caching in this case?  Revisit especially if making parallel / adding locks
				ComputedGridPts.Add(GID, ToReturn);
			}
		}
		else
		{
			if (GridPtData->VertexIndex >= 0)
			{
				PosOut = Vertices[GridPtData->VertexIndex];
			}
			else
			{
				PosOut = GetUnwarpedGridPos(Pt);
			}
			ToReturn = *GridPtData;
		}
		if (bAllowWarp && !ToReturn.bWarpProcessed)
		{
			// Test all connected edges and warp if needed, otherwise cache cut points
			int16 LabelA = ToReturn.Label;
			if (LabelA == 0)
			{
				ToReturn.bWarpProcessed = true; // already zero == already warped
				return ToReturn;
			}
			int64 EdgeIDs[14];
			FGridPt OtherPts[14];
			int64 CutPtIDs[14]; // EdgeIDs that *may* have computed cut points stored
			int32 HasCutPtNum = 0; // Track the added cut points, in case we need to clear them
			GetEdges(Pt, EdgeIDs, OtherPts);
			// Note AlphaLong, AlphaShort are "double-sided" weights taken from the Isosurface Stuffing paper
			// to allow inclusion of "all on surface" tets in the output without bad quality results
			// TODO: expose more weight options: we should be able to trade higher quality results if we use different weights and filter such tets heuristically
			constexpr RealType AlphaLong = 0.21509;
			constexpr RealType AlphaShort = 0.35900;
			RealType SmallestAlpha = 2; // Note: Intentionally larger than max valid Alpha value of 1
			int32 WarpEdgeIdx = INDEX_NONE;
			TVec3 WarpPos;
			auto EvalEdgeRange = [this, Pt, PosOut, LabelA, &EdgeIDs, &OtherPts](int32 MinEdgeIdx, int32 MaxEdgeIdx, RealType AlphaThreshold,
				// Note: The following are passed by reference explicitly, rather than via capture, because PVS static analysis didn't believe in the captures:
				RealType& SmallestAlphaRef, int32& WarpEdgeIdxRef, TVec3& WarpPosRef, int32& HasCutPtNumRef, int64* CutPtIDsArrPtr)
			{
				RealType KeepCutAlphaThreshold = WarpEdgeIdxRef == INDEX_NONE ? AlphaThreshold : (RealType)2;
				for (int32 EdgeIdx = MinEdgeIdx; EdgeIdx < MaxEdgeIdx; ++EdgeIdx)
				{
					TVec3 OtherPos;
					FGridPtData OtherData = GetGridPtData(OtherPts[EdgeIdx], OtherPos, false);
					FCutData CutData;
					RealType Alpha;
					bool bHasCut = GetCutData(EdgeIDs[EdgeIdx], Pt, OtherPts[EdgeIdx], PosOut, OtherPos, LabelA, OtherData.Label, KeepCutAlphaThreshold, CutData, Alpha);
					if (bHasCut)
					{
						CutPtIDsArrPtr[HasCutPtNumRef++] = EdgeIDs[EdgeIdx]; // TODO: make GetCutData report whether there is actually anything in the has, and only add to this array in that case
						if (Alpha < AlphaThreshold)
						{
							KeepCutAlphaThreshold = 2; // stop keeping cut points
							if (Alpha < SmallestAlphaRef)
							{
								WarpEdgeIdxRef = EdgeIdx;
								SmallestAlphaRef = Alpha;
								WarpPosRef = CutData.CutPos;
							}
						}
					}
				}
			};
			EvalEdgeRange(0, 6, AlphaLong, SmallestAlpha, WarpEdgeIdx, WarpPos, HasCutPtNum, CutPtIDs); // Process the long edges
			EvalEdgeRange(6, 14, AlphaShort, SmallestAlpha, WarpEdgeIdx, WarpPos, HasCutPtNum, CutPtIDs); // Process the short edges
			ToReturn.bWarpProcessed = true;
			if (WarpEdgeIdx != INDEX_NONE)
			{
				PosOut = WarpPos;
				for (int32 CutIDIdx = 0; CutIDIdx < HasCutPtNum; ++CutIDIdx)
				{
					ComputedCutPts.Remove(CutPtIDs[CutIDIdx]);
				}
				ToReturn.Label = 0;
				if (ToReturn.VertexIndex >= 0)
				{
					Vertices[ToReturn.VertexIndex] = WarpPos;
				}
				else
				{
					ToReturn.VertexIndex = Vertices.Add(WarpPos);
				}
			}
			// Add or replace the grid point data now that the possible warping has been processed
			ComputedGridPts.Add(GID, ToReturn);
		}
		return ToReturn;
	}

	static RealType GetAlpha(const TVec3& APos, const TVec3& BPos, const TVec3& Between)
	{
		TVec3 Edge = BPos - APos;
		RealType EdgeLenSq = Edge.Dot(Edge);
		RealType Alpha = Edge.Dot(Between - APos) / EdgeLenSq;
		return Alpha;
	}

	bool GetCutData(int64 EdgeID, const FGridPt& A, const FGridPt& B, const TVec3& APos, const TVec3& BPos, int16 LabelA, int16 LabelB, RealType AlphaThreshold, FCutData& CutOut, RealType& AlphaOut)
	{
		if (LabelB == 0 || LabelA == LabelB)
		{
			return false;
		}
		FCutData* CutData = ComputedCutPts.Find(EdgeID);
		if (CutData)
		{
			CutOut = *CutData;
			AlphaOut = GetAlpha(APos, BPos, CutOut.CutPos);
			return true;
		}
		else
		{
			RealType AVal = LabelToImplicitValue(LabelA);
			RealType BVal = LabelToImplicitValue(LabelB);
			CutOut.CutPos = ComputeEdgeCutPos(APos, AVal, BPos, BVal);
			CutOut.VertexIndex = INDEX_NONE; // Note: Don't create a vertex for the cut point yet, since it may be warped away
			AlphaOut = GetAlpha(APos, BPos, CutOut.CutPos);
			if (AlphaOut > AlphaThreshold)
			{
				// TODO: Consider not adding to map here, and updating the map only after we've verified no warping locally
				ComputedCutPts.Add(EdgeID, CutOut);
			}
			return true;
		}
	}

	int32 GetOrAddCutVertex(const FGridPt& A, const FGridPt& B)
	{
		int64 EID = GetEdgeID(A, B);
		FCutData* CutData = ComputedCutPts.Find(EID);
		check(CutData); // TODO: make sure this function can only be called when there is definitely cut data cached for the edge?
		if (CutData->VertexIndex == INDEX_NONE)
		{
			CutData->VertexIndex = Vertices.Add(CutData->CutPos);
		}
		return CutData->VertexIndex;
	}

	// TODO: should this take the pre-fetched edge data as well?
	// Stuff output tetrahedra into the given background tetrahedron
	void ProcessTet(FGridPt Pts[4], FGridPtData Data[4])
	{
		int32 Order[4]{ -1,-1,-1,-1 }; // storage for index re-ordering, to apply templates
		int32 CutV[4]{ -1,-1,-1,-1 }; // storage for vertex indices of cut vertices
		// For the last 5 templates, we break the tet into the two long edges connected by the first and second pair of vertices, 0-1, 2-3
		// OtherOfPair goes from the index of one side of a long edge to the other side of the same long edge (e.g., 0<->1, 2<->3)
		// OtherPair goes from the index of one long edge to the corresponding (1st or 2nd) vertex on the other long edge (e.g., 0<->2, 1<->3)
		auto OtherOfPair = [](int32 SubIdx)
		{
			return SubIdx ^ 1;
		};
		auto OtherPair = [](int32 SubIdx)
		{
			return SubIdx ^ 2;
		};
		// Helper for the "Parity" test to disambiguate triangulation of quads for the last 3 stencils
		// See Isosurface Stuffing paper Sec 3.3 and Fig 3 for details
		auto NeedsFlipForParityTest = [](const int32 Order0, const TVec3& A, const TVec3& C) -> bool
		{
			// by convention, first two vertices are centers, second two are corners, so Order0 being 2 or 3 -> it's on a corner
			int32 EdgeABConnectsCorners = ((Order0 & 2) >> 1);
			int32 DiagonalACHasOddParity = ((int32(A.X > C.X) + int32(A.Y > C.Y) + int32(A.Z > C.Z)) & 1);
			return EdgeABConnectsCorners == DiagonalACHasOddParity;
		};

		// Counts and indices for vertex labels, to choose which template to apply
		int32 NumOut = 0, NumOn = 0, NumIn = 0;
		int32 NegIdx = INDEX_NONE, ZeroIdx = INDEX_NONE, PosIdx = INDEX_NONE;
		for (int32 GridPtIdx = 0; GridPtIdx < 4; ++GridPtIdx)
		{
			int16 Label = Data[GridPtIdx].Label;
			if (Label < 0)
			{
				NegIdx = GridPtIdx;
				NumOut++;
			}
			else if (Label == 0)
			{
				ZeroIdx = GridPtIdx;
				NumOn++;
			}
			else
			{
				PosIdx = GridPtIdx;
				NumIn++;
			}
		}

		if (NumIn + NumOn == 4) // First 4 cases: all in or on
		{
			if (NumOn == 4)
			{
				TVec3 Avg = (Vertices[Data[0].VertexIndex] + Vertices[Data[1].VertexIndex] + Vertices[Data[2].VertexIndex] + Vertices[Data[3].VertexIndex]) * .25;
				int16 Label = ImplicitValueToLabel(Implicit(Avg));
				if (Label < 0)
				{
					return; // centroid outside -> skip tet
				}
			}
			this->AppendTet(Data[0].VertexIndex, Data[1].VertexIndex, Data[2].VertexIndex, Data[3].VertexIndex);
		}
		else if (NumIn == 1) // Mid 3 cases: Only on in, at least one out
		{
			FIntVector4 TetV;
			for (int32 TetIdx = 0; TetIdx < 4; ++TetIdx)
			{
				if (Data[TetIdx].Label < 0)
				{
					TetV[TetIdx] = GetOrAddCutVertex(Pts[TetIdx], Pts[PosIdx]);
				}
				else
				{
					TetV[TetIdx] = Data[TetIdx].VertexIndex;
				}
			}
			this->AppendTet(TetV.X, TetV.Y, TetV.Z, TetV.W);
		}
		else if (NumIn == 0) // all negative or zero (w/ at least one negative) == empty
		{
			return;
		}
		// Last 5 Cases; See Fig 3 in the Isosurface Stuffing paper
		else if (NumOn == 1) // 1st or 3rd of the last cases
		{
			if (Data[0].Label == Data[1].Label || Data[2].Label == Data[3].Label) // 3rd case
			{
				Order[2] = ZeroIdx;
				Order[3] = OtherOfPair(ZeroIdx);
				Order[1] = OtherPair(ZeroIdx);
				Order[0] = OtherOfPair(Order[1]);
				CutV[0] = GetOrAddCutVertex(Pts[Order[3]], Pts[Order[0]]);
				CutV[1] = GetOrAddCutVertex(Pts[Order[3]], Pts[Order[1]]);
				// Check if parity test indicates we need to flip the first two (positive) vertices of the stencil
				bool bFlipped = NeedsFlipForParityTest(Order[0], Vertices[Data[Order[0]].VertexIndex], Vertices[CutV[1]]);
				if (bFlipped)
				{
					Swap(Order[0], Order[1]);
					Swap(CutV[0], CutV[1]);
				}
				// Verify template order ++0-
				checkSlow(Data[Order[0]].Label > 0 && Data[Order[1]].Label > 0 && Data[Order[2]].Label == 0 && Data[Order[3]].Label < 0);
				FIntVector4 TetVA(Data[Order[1]].VertexIndex, Data[Order[2]].VertexIndex, CutV[1], CutV[0]);
				FIntVector4 TetVB(Data[Order[0]].VertexIndex, Data[Order[1]].VertexIndex, CutV[0], Data[Order[2]].VertexIndex);
				this->AppendTetOriented(TetVA, bFlipped);
				this->AppendTetOriented(TetVB, bFlipped);
			}
			else // 1st case (0+, +-)
			{
				bool bFlipped = false;
				Order[0] = ZeroIdx;
				Order[1] = OtherOfPair(ZeroIdx);
				Order[2] = OtherPair(ZeroIdx);
				Order[3] = OtherOfPair(Order[2]);
				if (Data[Order[2]].Label < 0)
				{
					Swap(Order[2], Order[3]);
					bFlipped = !bFlipped;
				}
				// Verify template order 0++-
				checkSlow(Data[Order[0]].Label == 0 && Data[Order[1]].Label > 0 && Data[Order[2]].Label > 0 && Data[Order[3]].Label < 0);
				CutV[0] = GetOrAddCutVertex(Pts[Order[2]], Pts[Order[3]]);
				CutV[1] = GetOrAddCutVertex(Pts[Order[3]], Pts[Order[1]]);
				FIntVector4 TetVA(Data[Order[0]].VertexIndex, Data[Order[1]].VertexIndex, CutV[0], CutV[1]);
				FIntVector4 TetVB(Data[Order[0]].VertexIndex, Data[Order[1]].VertexIndex, Data[Order[2]].VertexIndex, CutV[0]);
				this->AppendTetOriented(TetVA, bFlipped);
				this->AppendTetOriented(TetVB, bFlipped);
			}
		}
		else if (NumIn == 3) // 4th case
		{
			Order[3] = NegIdx;
			Order[2] = OtherOfPair(NegIdx);
			Order[1] = OtherPair(NegIdx);
			Order[0] = OtherOfPair(Order[1]);
			CutV[0] = GetOrAddCutVertex(Pts[Order[0]], Pts[Order[3]]);
			CutV[1] = GetOrAddCutVertex(Pts[Order[1]], Pts[Order[3]]);
			// Check if parity test indicates we need to flip the first two (positive) vertices of the stencil
			bool bFlipped = NeedsFlipForParityTest(Order[0], Vertices[Data[Order[0]].VertexIndex], Vertices[CutV[1]]);
			if (bFlipped)
			{
				Swap(Order[0], Order[1]);
				Swap(CutV[0], CutV[1]);
			}
			// Verify template order +++-
			checkSlow(Data[Order[0]].Label > 0 && Data[Order[1]].Label > 0 && Data[Order[2]].Label > 0 && Data[Order[3]].Label < 0);
			CutV[2] = GetOrAddCutVertex(Pts[Order[2]], Pts[Order[3]]);
			FIntVector4 TetVA(CutV[0], Data[Order[0]].VertexIndex, CutV[2], Data[Order[1]].VertexIndex);
			FIntVector4 TetVB(CutV[2], Data[Order[0]].VertexIndex, Data[Order[2]].VertexIndex, Data[Order[1]].VertexIndex);
			FIntVector4 TetVC(CutV[0], CutV[1], Data[Order[1]].VertexIndex, CutV[2]);
			this->AppendTetOriented(TetVA, bFlipped);
			this->AppendTetOriented(TetVB, bFlipped);
			this->AppendTetOriented(TetVC, bFlipped);
		}
		else if (Data[0].Label == Data[1].Label) // 5th case
		{
			Order[0] = PosIdx;
			Order[1] = OtherOfPair(PosIdx);
			Order[2] = OtherPair(PosIdx);
			Order[3] = OtherOfPair(Order[2]);
			CutV[0] = GetOrAddCutVertex(Pts[Order[0]], Pts[Order[3]]);
			CutV[1] = GetOrAddCutVertex(Pts[Order[1]], Pts[Order[3]]);
			// Check if parity test indicates we need to flip the first two (positive) vertices of the stencil
			bool bFlipped = NeedsFlipForParityTest(Order[0], Vertices[Data[Order[0]].VertexIndex], Vertices[CutV[1]]);
			if (bFlipped)
			{
				Swap(Order[0], Order[1]);
				Swap(CutV[0], CutV[1]);
			}
			// Verify template order ++--
			checkSlow(Data[Order[0]].Label > 0 && Data[Order[1]].Label > 0 && Data[Order[2]].Label < 0 && Data[Order[3]].Label < 0);
			// Compute these last two cut vertices *after* the potential parity swap, so we don't need to swap them
			CutV[2] = GetOrAddCutVertex(Pts[Order[1]], Pts[Order[2]]);
			CutV[3] = GetOrAddCutVertex(Pts[Order[0]], Pts[Order[2]]);
			FIntVector4 TetVA(CutV[0], Data[Order[1]].VertexIndex, Data[Order[0]].VertexIndex, CutV[2]);
			FIntVector4 TetVB(CutV[0], Data[Order[1]].VertexIndex, CutV[2], CutV[1]);
			FIntVector4 TetVC(CutV[0], CutV[3], CutV[2], Data[Order[0]].VertexIndex);
			this->AppendTetOriented(TetVA, bFlipped);
			this->AppendTetOriented(TetVB, bFlipped);
			this->AppendTetOriented(TetVC, bFlipped);
		}
		else // 2nd case
		{
			checkSlow(Data[0].Label != Data[1].Label);
			bool bFlipped = false;
			Order[0] = NegIdx;
			Order[1] = OtherOfPair(NegIdx);
			Order[2] = OtherPair(NegIdx);
			Order[3] = OtherOfPair(Order[2]);
			if (Data[Order[2]].Label > 0)
			{
				Swap(Order[2], Order[3]);
				bFlipped = !bFlipped;
			}
			// Verify template order -+-+
			checkSlow(Data[Order[0]].Label < 0 && Data[Order[1]].Label > 0 && Data[Order[2]].Label < 0 && Data[Order[3]].Label > 0);
			CutV[0] = GetOrAddCutVertex(Pts[Order[0]], Pts[Order[1]]);
			CutV[1] = GetOrAddCutVertex(Pts[Order[1]], Pts[Order[2]]);
			CutV[2] = GetOrAddCutVertex(Pts[Order[2]], Pts[Order[3]]);
			CutV[3] = GetOrAddCutVertex(Pts[Order[3]], Pts[Order[0]]);
			FIntVector4 TetVA(CutV[2], Data[Order[1]].VertexIndex, CutV[0], CutV[1]);
			FIntVector4 TetVB(CutV[3], Data[Order[3]].VertexIndex, CutV[0], CutV[2]);
			FIntVector4 TetVC(CutV[0], Data[Order[1]].VertexIndex, CutV[2], Data[Order[3]].VertexIndex);
			this->AppendTetOriented(TetVA, bFlipped);
			this->AppendTetOriented(TetVB, bFlipped);
			this->AppendTetOriented(TetVC, bFlipped);
		}
	}
};

}} // namespace UE::TetMeshing