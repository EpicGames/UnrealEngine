// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VectorTypes.h"
#include "Math/IntVector.h"

namespace UE
{
namespace Geometry
{

/**
 * Base class for tetrahedral mesh generators (eg like to generate sphere, cylinder, etc)
 * Subclasses must implement ::Generate() 
 */
template<typename RealType>
class TTetMeshGenerator
{
public:
	using TVec3 = UE::Math::TVector<RealType>;

	/** Array of vertex positions */
	TArray<TVec3> Vertices;

	/** Array of tetrahedron corner positions, stored as tuples of indices into Vertices array */
	TArray<FIntVector4> Tets;
	// Optional array of tetrahedron IDs.  If non-empty, should be same length as Tets array.
	TArray<int32> TetIDs;
	bool bCreateTetIDs = false;

	// Optional array of triangle corner indices, which should correspond to faces of existing tetrahedra
	TArray<FIntVector3> Triangles;
	// Optional array of ID tags for triangles. If non-empty, should be same length as Triangles array.
	TArray<int32> TriangleIDs;
	bool bCreateTriangleIDs = false;


public:
	/** If true, reverse orientation of created mesh */
	bool bReverseOrientation = false;

	virtual ~TTetMeshGenerator()
	{
	}


	/**
	 * Subclasses implement this to generate mesh
	 */
	virtual TTetMeshGenerator& Generate() = 0;


	/** clear arrays so that Generate() can be run again */
	virtual void Reset()
	{
		Vertices.Reset();
		Tets.Reset();
		TetIDs.Reset();
		Triangles.Reset();
		TriangleIDs.Reset();
	}


	/**
	 * Set the various internal buffers to the correct sizes for the given element counts
	 */
	void SetBufferSizes(int32 NumVertices, int32 NumTetrahedra, bool bIncludeTetIDs, int32 NumTriangles, bool bIncludeTriangleIDs)
	{
		Vertices.SetNum(NumVertices);
		Tets.SetNum(NumTetrahedra);
		if (bIncludeTetIDs)
		{
			bCreateTetIDs = bIncludeTetIDs;
			TetIDs.SetNum(NumTetrahedra);
		}
		Triangles.SetNum(NumTriangles);
		if (bIncludeTriangleIDs)
		{
			bCreateTriangleIDs = bIncludeTriangleIDs;
			TriangleIDs.SetNum(NumTriangles);
		}
	}
	/**
	 * Extends the various internal buffers to the correct sizes for the given additional element counts
	 */
	void ExtendBufferSizes(int32 AddVertices, int32 AddTriangles, int32 AddTets)
	{
		if (AddVertices > 0)
		{
			int32 NumVertices = Vertices.Num() + AddVertices;
			Vertices.SetNum(NumVertices);
		}
		if (AddTriangles > 0)
		{
			int32 NumTriangles = Triangles.Num() + AddTriangles;
			Triangles.SetNum(NumTriangles);
			if (bCreateTriangleIDs)
			{
				TriangleIDs.SetNum(NumTriangles);
			}
		}
		if (AddTets > 0)
		{
			int32 NumTets = Tets.Num() + AddTets;
			Tets.SetNum(NumTets);
			if (bCreateTetIDs)
			{
				TetIDs.SetNum(NumTets);
			}
		}
	}

	
	inline int32 AppendTet(int32 A, int32 B, int32 C, int32 D, int32 TetID = -1)
	{
		if (bCreateTetIDs)
		{
			TetIDs.Add(TetID);
		}
		return Tets.Add( bReverseOrientation ? FIntVector4(A, C, B, D) : FIntVector4(A, B, C, D) );
	}

	inline int32 AppendTetOriented(FIntVector4 Tet, bool bFlipOrientation, int32 TetID = -1)
	{
		if (bCreateTetIDs)
		{
			TetIDs.Add(TetID);
		}
		return Tets.Add((bFlipOrientation ^ bReverseOrientation) ? FIntVector4(Tet.X, Tet.Z, Tet.Y, Tet.W) : FIntVector4(Tet.X, Tet.Y, Tet.Z, Tet.W));
	}

};

} // end namespace UE::Geometry
} // end namespace UE