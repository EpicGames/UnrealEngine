// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class TetMeshing : ModuleRules
	{
        public TetMeshing(ReadOnlyTargetRules Target) : base(Target)
		{
            PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"MeshUtilitiesCommon",
					"StaticMeshDescription",
					"MeshConversion",
				}
				);
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"GeometryCore",
					"DynamicMesh",
					"Chaos",
					"GeometryAlgorithms",
					"MeshDescription",
                }
                );
		}
	}
}
