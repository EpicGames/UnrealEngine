{
    "FileVersion" : 3,
    "Version" : 1,
    "VersionName" : "0.1",
    "FriendlyName" : "Tet Meshing",
    "Description" : "Adds Module for Generating and Refining Tetrahedral Meshes.",
    "Category": "Physics",
    "CreatedBy" : "Epic Games, Inc.",
    "CreatedByURL" : "https://epicgames.com",
    "DocsURL" : "",
    "MarketplaceURL" : "",
    "SupportURL" : "",
    "EnabledByDefault" : false,
    "CanContainContent" : false,
    "IsBetaVersion" : true,
    "Installed" : false,
    "Modules": [
      {
        "Name": "TetMeshing",
        "Type": "Runtime",
        "LoadingPhase": "PostEngineInit"
      }
    ],
    "Plugins": [
        {
            "Name": "GeometryProcessing",
            "Enabled": true
        }
    ]
}
